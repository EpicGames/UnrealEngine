// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "IExternalImagePickerModule.h"
#include "SExternalImagePicker.h"

/**
 * Public interface for splash screen editor module
 */
class FExternalImagePickerModule : public IExternalImagePickerModule
{
public:
	virtual TSharedRef<SWidget> MakeEditorWidget(const FExternalImagePickerConfiguration& InConfiguration) override
	{
		return SNew(SExternalImagePicker)
			.TargetImagePath(InConfiguration.TargetImagePath)
			.DefaultImagePath(InConfiguration.DefaultImagePath)
			.OnExternalImagePicked(InConfiguration.OnExternalImagePicked)
			.OnGetPickerPath(InConfiguration.OnGetPickerPath)
			.MaxDisplayedImageDimensions(InConfiguration.MaxDisplayedImageDimensions)
			.RequiresSpecificSize(InConfiguration.bRequiresSpecificSize)
			.RequiredImageDimensions(InConfiguration.RequiredImageDimensions)
			.Extensions(InConfiguration.FileExtensions)
			.GenerateImageVisibility(InConfiguration.GenerateImageVisibility)
			.GenerateImageToolTipText(InConfiguration.GenerateImageToolTipText)
			.OnGenerateImageClicked(InConfiguration.OnGenerateImageClicked);
	}
};

IMPLEMENT_MODULE(FExternalImagePickerModule, ExternalImagePicker)
