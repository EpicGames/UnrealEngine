// Copyright Epic Games, Inc. All Rights Reserved.

#include "SExternalImagePicker.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "SResetToDefaultMenu.h"
#include "DesktopPlatformModule.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Widgets/Layout/SEnableBox.h"

#define LOCTEXT_NAMESPACE "ExternalImagePicker"

void SExternalImagePicker::Construct(const FArguments& InArgs)
{
	TargetImagePath = InArgs._TargetImagePath;
	DefaultImagePath = InArgs._DefaultImagePath;
	Extensions = InArgs._Extensions;

	FString TargetImagePathExtension = FPaths::GetExtension(TargetImagePath);
	if (!Extensions.Contains(TargetImagePathExtension))
	{
		Extensions.Add(TargetImagePathExtension);
	}

	OnExternalImagePicked = InArgs._OnExternalImagePicked;
	OnGetPickerPath = InArgs._OnGetPickerPath;
	MaxDisplayedImageDimensions = InArgs._MaxDisplayedImageDimensions;
	bRequiresSpecificSize = InArgs._RequiresSpecificSize;
	RequiredImageDimensions = InArgs._RequiredImageDimensions;
	
	TSharedPtr<SHorizontalBox> HorizontalBoxWidget = nullptr;

    ChildSlot
           [
               SNew(SVerticalBox)
               +SVerticalBox::Slot()
               [
                   SAssignNew(HorizontalBoxWidget, SHorizontalBox)
                   +SHorizontalBox::Slot()
                   .AutoWidth()
                   .VAlign(VAlign_Center)
                   [
                       SNew(SBorder)
                       .BorderImage(FAppStyle::Get().GetBrush("ExternalImagePicker.ThumbnailShadow"))
                       .Padding(4.0f)
                       .Content()
                       [
                           SNew(SBorder)
                           .BorderImage(FAppStyle::Get().GetBrush("ExternalImagePicker.BlankImage"))
                           .Padding(0.0f)
                           .Content()
                           [
                               SNew(SBox)
                               .WidthOverride(this, &SExternalImagePicker::GetImageWidth)
                               .HeightOverride(this, &SExternalImagePicker::GetImageHeight)
                               [
                                   SNew(SEnableBox)
                                   [
                                       SNew(SImage)
                                       .Image(this, &SExternalImagePicker::GetImage)
                                       .ToolTipText(this, &SExternalImagePicker::GetImageTooltip)
                                   ]
                               ]
                           ]
                       ]
                   ]
                   +SHorizontalBox::Slot()
                   .AutoWidth()
                   .Padding(2.0f)
                   .VAlign(VAlign_Center)
                   [
                       SNew(SButton)
                       .ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
                       .ToolTipText( LOCTEXT( "FileButtonToolTipText", "Choose a file from this computer") )
                       .OnClicked( FOnClicked::CreateSP(this, &SExternalImagePicker::OnPickFile) )
                       .ContentPadding( 2.0f )
                       .ForegroundColor( FSlateColor::UseForeground() )
                       .IsFocusable( false )
                       [
                           SNew( SImage )
                           .Image( FAppStyle::Get().GetBrush("ExternalImagePicker.PickImageButton") )
                           .ColorAndOpacity( FSlateColor::UseForeground() )
                       ]
                   ]
				   +SHorizontalBox::Slot()
				  .AutoWidth()
				  .Padding(2.0f)
				  .VAlign(VAlign_Center)
				   [
					  SNew( SButton )
						.Visibility(InArgs._GenerateImageVisibility)
						.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
						.ToolTipText(InArgs._GenerateImageToolTipText )
						.OnClicked(FOnClicked::CreateSP(this, &SExternalImagePicker::OnGenerateImageClickedInternal, InArgs._OnGenerateImageClicked))
						.ContentPadding( 2.0f )
						.ForegroundColor( FSlateColor::UseForeground() )
						.IsFocusable( false )
						[
							SNew( SImage )
							.Image( FAppStyle::Get().GetBrush("ExternalImagePicker.GenerateImageButton") )
							.ColorAndOpacity( FSlateColor::UseForeground() )
						]
					]
               ]
               +SVerticalBox::Slot()
               .AutoHeight()
               [
                   SAssignNew(ErrorHintWidget, SErrorText)
               ]
           ];
	if(HorizontalBox.IsValid() && DefaultImagePath.Len() > 0)
	{
		HorizontalBoxWidget->AddSlot()
			.AutoWidth()
			.Padding(2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SResetToDefaultMenu)
				.OnResetToDefault(FSimpleDelegate::CreateSP(this, &SExternalImagePicker::HandleResetToDefault))
				.OnGetResetToDefaultText(FOnGetResetToDefaultText::CreateSP(this, &SExternalImagePicker::GetResetToDefaultText))
				.DiffersFromDefault(this, &SExternalImagePicker::DiffersFromDefault)
			];
	}

	ApplyFirstValidImage();
}

const FSlateBrush* SExternalImagePicker::GetImage() const
{
	return ImageBrush.IsValid() ? ImageBrush.Get() : FAppStyle::Get().GetBrush("ExternalImagePicker.BlankImage");
}

FText SExternalImagePicker::GetImageTooltip() const
{
	const FVector2D Size = ImageBrush.IsValid() ? ImageBrush->ImageSize : FVector2D::ZeroVector;
	
	FText XText = FText::AsNumber((int32)Size.X);
	FText YText = FText::AsNumber((int32)Size.Y);

	FText ImageTooltip;
	switch (TypeOfImage)
	{
	case UsingDummyPlaceholderImage:
		ImageTooltip = LOCTEXT("ImageTooltip_Missing", "Warning: No Image Available!");
		break;
	case UsingDefaultImage:
		ImageTooltip = FText::Format(LOCTEXT("ImageTooltip_Default", "Default Image\n({0})\nDimensions: {1} x {2}"), FText::FromString(DefaultImagePath), XText, YText);
		break;
	case UsingTargetImage:
		ImageTooltip = FText::Format(LOCTEXT("ImageTooltip_Target", "Target Image\n({0})\nDimensions: {1} x {2}"), FText::FromString(TargetImagePath), XText, YText);
		break;
	}
	
	return ImageTooltip;
}

FVector2D SExternalImagePicker::GetConstrainedImageSize() const
{
	const FVector2D Size = GetImage()->ImageSize;

	// make sure we have a valid size in case the image didn't get loaded
	const FVector2D ValidSize( FMath::Max(Size.X, 32.0f), FMath::Max(Size.Y, 32.0f) );

	// keep image aspect but don't display it above a reasonable size
	const float ImageAspect = ValidSize.X / ValidSize.Y;
	const FVector2D ConstrainedSize( FMath::Min(ValidSize.X, MaxDisplayedImageDimensions.X), FMath::Min(ValidSize.Y, MaxDisplayedImageDimensions.Y) );
	return FVector2D( FMath::Min(ConstrainedSize.X, ConstrainedSize.Y * ImageAspect), FMath::Min(ConstrainedSize.Y, ConstrainedSize.X / ImageAspect) );
}

FOptionalSize SExternalImagePicker::GetImageWidth() const
{
	return GetConstrainedImageSize().X;
}

FOptionalSize SExternalImagePicker::GetImageHeight() const
{
	return GetConstrainedImageSize().Y;
}

void SExternalImagePicker::ApplyImageWithExtenstion(const FString& Extension)
{
	//Remove the target image path's old extension
	TargetImagePath = FPaths::GetPath(TargetImagePath) / FPaths::GetBaseFilename(TargetImagePath) + TEXT(".");
	TargetImagePath.Append(Extension);

	ApplyImage();
}

void SExternalImagePicker::ApplyFirstValidImage()
{
	FString NewTargetImagePathNoExtension = FPaths::GetPath(TargetImagePath) / FPaths::GetBaseFilename(TargetImagePath) + TEXT(".");

	for (FString Ex : Extensions)
	{
		FString NewTargetImagePath = NewTargetImagePathNoExtension + Ex;

		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*NewTargetImagePath))
		{
			TargetImagePath = NewTargetImagePath;
			break;
		}
	}

	ApplyImage();
}

void SExternalImagePicker::ApplyImage()
{
	ErrorHintWidget->SetError(FText::GetEmpty());
	if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*TargetImagePath))
	{
		TypeOfImage = UsingTargetImage;
		ApplyImage(TargetImagePath);
	}
	else if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*DefaultImagePath))
	{
		TypeOfImage = UsingDefaultImage;
		ApplyImage(DefaultImagePath);
	}
	else
	{
        // This is preventing the "no default image" error display when we don't want a fallback behavior, i.e. iOS optional icons
        FString DefaultImagePathString(DefaultImagePath);
        if (DefaultImagePathString.IsEmpty())
        {
            return;
        }
            
		TypeOfImage = UsingDummyPlaceholderImage;

		if (bRequiresSpecificSize)
		{
			ErrorHintWidget->SetError(FText::Format(
				LOCTEXT("BadSizeNoImageHint", "No image at '{0}' ({1}x{2})"),
				FText::FromString(TargetImagePath),
				FText::AsNumber(RequiredImageDimensions.X),
				FText::AsNumber(RequiredImageDimensions.Y)
				));
		}
		else
		{
			ErrorHintWidget->SetError(FText::Format(LOCTEXT("NoImageErrorHint", "No image at '{0}'"), FText::FromString(TargetImagePath)));
		}

		ApplyPlaceholderImage();
	}
}

void SExternalImagePicker::ApplyImage(const FString& ImagePath)
{
	if (ImageBrush.IsValid())
	{
		FSlateApplication::Get().GetRenderer()->ReleaseDynamicResource(*ImageBrush.Get());
		ImageBrush.Reset();
	}

	ImageBrush = LoadImageAsBrush(ImagePath);
}

void SExternalImagePicker::ApplyPlaceholderImage()
{
	if (ImageBrush.IsValid())
	{
		FSlateApplication::Get().GetRenderer()->ReleaseDynamicResource(*ImageBrush.Get());
	}

	ImageBrush.Reset();
}

TSharedPtr< FSlateDynamicImageBrush > SExternalImagePicker::LoadImageAsBrush( const FString& ImagePath )
{
	TSharedPtr< FSlateDynamicImageBrush > Brush;
	bool bSucceeded = false;
	
	TArray64<uint8> RawFileData;
	if( FFileHelper::LoadFileToArray( RawFileData, *ImagePath ) )
	{
		IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>( FName("ImageWrapper") );

		FImage Image;
		if ( ImageWrapperModule.DecompressImage(RawFileData.GetData(), RawFileData.Num(), Image) )
		{
			Image.ChangeFormat(ERawImageFormat::BGRA8,EGammaSpace::sRGB);

			// have a TArray64 but API needs a TArray
			TArray<uint8> ImageRawData32( MoveTemp(Image.RawData) );
			if (FSlateApplication::Get().GetRenderer()->GenerateDynamicImageResource(*ImagePath, Image.SizeX, Image.SizeY, ImageRawData32))
			{
				Brush = MakeShareable(new FSlateDynamicImageBrush(*ImagePath, FVector2D(Image.SizeX, Image.SizeY)));
				bSucceeded = true;
			}
		}

		if (!bSucceeded)
		{
			UE_LOGF(LogSlate, Log, "External Image Picker: DecompressImage failed");
			ErrorHintWidget->SetError(LOCTEXT("BadFormatHint", "Unsupported image format"));
		}
		else
		{
			if (bRequiresSpecificSize)
			{
				if (Brush->ImageSize != RequiredImageDimensions)
				{
					ErrorHintWidget->SetError(FText::Format(
						LOCTEXT("BadSizeHint", "Incorrect size ({0}x{1} but should be {2}x{3})"),
						FText::AsNumber((int32)Brush->ImageSize.X),
						FText::AsNumber((int32)Brush->ImageSize.Y),
						FText::AsNumber(RequiredImageDimensions.X),
						FText::AsNumber(RequiredImageDimensions.Y)
					));
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogSlate, Log, "Could not find file for image: %ls", *ImagePath);
	}

	return Brush;
}


FReply SExternalImagePicker::OnPickFile()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		FText Title;
		FString TitleExtensions;
		FString AssociatedExtensions;
		if (Extensions.Num() > 1)
		{
			Title = LOCTEXT("Image", "Image");
			TitleExtensions = TEXT("*.") + FString::Join(Extensions, TEXT(", *."));
			AssociatedExtensions = TitleExtensions.Replace(TEXT(", "), TEXT(";"));
		}
		else
		{
			Title = FText::FromString(Extensions[0]);
			TitleExtensions = TEXT("*.") + Extensions[0];
			AssociatedExtensions = TitleExtensions;
		}

		TArray<FString> OutFiles;
		const FString Filter = FString::Printf(TEXT("%s files (%s)|%s"), *Title.ToString(), *TitleExtensions, *AssociatedExtensions);
		const FString DefaultPath = OnGetPickerPath.IsBound() ? OnGetPickerPath.Execute() : FPaths::GetPath(FPaths::GetProjectFilePath());

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		if (DesktopPlatform->OpenFileDialog(ParentWindowHandle, FText::Format(LOCTEXT("ImagePickerDialogTitle", "Choose a {0} file"), Title).ToString(), DefaultPath, TEXT(""), Filter, EFileDialogFlags::None, OutFiles))
		{
			check(OutFiles.Num() == 1);

			// do not reload image if it's the same path 
			FString SourceImagePath = FPaths::ConvertRelativePathToFull(OutFiles[0]);
			if (SourceImagePath != TargetImagePath)
			{
				if ( ! OnExternalImagePicked.Execute(SourceImagePath, TargetImagePath) )
				{
					// can fail due to source control and other things
					
					UE_LOGF(LogSlate, Warning, "OnExternalImagePicked.Execute failed : %ls, %ls", *SourceImagePath, *TargetImagePath);
				}
				else
				{
					ApplyImageWithExtenstion(FPaths::GetExtension(SourceImagePath));
				}
			}
		}
	}

	return FReply::Handled();
}

FReply SExternalImagePicker::OnGenerateImageClickedInternal(FOnClicked UserOnGenerateImageClicked)
{
	if (UserOnGenerateImageClicked.IsBound())
	{
		if (!UserOnGenerateImageClicked.Execute().IsEventHandled())
		{
			ErrorHintWidget->SetError(LOCTEXT("IconGenerationFailed", "Image generation failed"));
		}
		else
		{
			ApplyImage();
		}
	}
	return FReply::Handled();
}

void SExternalImagePicker::HandleResetToDefault()
{
	if(OnExternalImagePicked.Execute(DefaultImagePath, TargetImagePath))
	{
		ApplyImage();
	}
}

FText SExternalImagePicker::GetResetToDefaultText() const
{
	return FText::FromString(DefaultImagePath);
}

bool SExternalImagePicker::DiffersFromDefault() const
{
	return TargetImagePath != DefaultImagePath;
}

#undef LOCTEXT_NAMESPACE
