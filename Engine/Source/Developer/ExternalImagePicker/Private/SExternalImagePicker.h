// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Types/SlateStructs.h"
#include "Widgets/SCompoundWidget.h"
#include "IExternalImagePickerModule.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/Notifications/SErrorText.h"

/**
 * Widget for displaying and editing an external image reference (e.g., splash screen, platform icons, etc...)
 */
class SExternalImagePicker : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SExternalImagePicker ){}

		/** The path to the image we will be editing */
		SLATE_ARGUMENT(FString, TargetImagePath)

		/** The path to the default image to display */
		SLATE_ARGUMENT(FString, DefaultImagePath)

		/** Delegate fired when an image is picked */
		SLATE_ARGUMENT(FOnExternalImagePicked, OnExternalImagePicked)

		/** Delegate fired to get the path to start picking from */
		SLATE_ARGUMENT(FOnGetPickerPath, OnGetPickerPath)

		/** The dimensions the image display should be constrained to. Aspect ratio is maintained. */
		SLATE_ARGUMENT(FVector2D, MaxDisplayedImageDimensions)

		/** The size the actual image needs to be (ignored unless bRequiresSpecificSize is set) */
		SLATE_ARGUMENT(FIntPoint, RequiredImageDimensions)

		/** Does the image need to be a specific size? */
		SLATE_ARGUMENT(bool, RequiresSpecificSize)

		/** Extensions that the image is allowed to have */
		SLATE_ARGUMENT(TArray<FString>, Extensions)
    
		/** Whether to show the generate image button */
		SLATE_ATTRIBUTE(EVisibility, GenerateImageVisibility)

		/** Tooltip for Generate Image button */
		SLATE_ARGUMENT(FText, GenerateImageToolTipText);

		/** Delegate fired when an Generate Image button is pressed */
		SLATE_ARGUMENT(FOnClicked, OnGenerateImageClicked)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Applies the target image with the given extension to the preview, or the default if the target is missing */
	void ApplyImageWithExtenstion(const FString& Extension);

	/** Applies the first valid image that matches one of the supported extensions */
	void ApplyFirstValidImage();

	/** Applies the target image to the preview, or the default if the target is missing */
	void ApplyImage();

	/** Applies the image to the preview */
	void ApplyImage(const FString& ImagePath);

	/** Applies image used to indicate 'no image' */
	void ApplyPlaceholderImage();

	/** Delegate invoked when the 'pick file' button is clicked */
	FReply OnPickFile();
    
    /** Internal delegate invoked when the 'generate image' button is clicked, triggers supplied user delegate */
    FReply OnGenerateImageClickedInternal(FOnClicked UserOnGenerateImageClicked);

	/** Delegate to retrieve the image to display */
	const FSlateBrush* GetImage() const;

	/** Returns a tooltip for the selected image, indicating the size */
	FText GetImageTooltip() const;

	/** Helper function to resize the image while retaining aspect ratio */
	FVector2D GetConstrainedImageSize() const;

	/** Delegate used to constrain the width of the displayed image */
	FOptionalSize GetImageWidth() const;

	/** Delegate used to constrain the height of the displayed image */
	FOptionalSize GetImageHeight() const;

	/** Load an image off disk & make a Slate brush out of it */
	TSharedPtr< FSlateDynamicImageBrush > LoadImageAsBrush( const FString& ImagePath );

	/** Delegate handler for resetting the image to default */
	void HandleResetToDefault();

	/** Delegate handler for getting the text to use when resetting to default */
	FText GetResetToDefaultText() const;

	/** Check to see if we differ from the default */
	bool DiffersFromDefault() const;

private:
    TSharedPtr<SHorizontalBox> HorizontalBox;
    
	/** The brush we use to draw the splash */
	TSharedPtr<FSlateDynamicImageBrush> ImageBrush;

	/** The box that contains the image preview */
	TSharedPtr<SHorizontalBox> ImageBox;

	/** The path to the default image to display */
	FString DefaultImagePath;

	/** The path to the image we will be editing */
	FString TargetImagePath;

	/** The extensions of the file types we want to use */
	TArray<FString> Extensions;

	/** Delegate fired when an image is picked */
	FOnExternalImagePicked OnExternalImagePicked;

	/** The path the picker will use to start from */
	FOnGetPickerPath OnGetPickerPath;

	/** The kind of image being used */
	enum EFileLocation
	{
		UsingDummyPlaceholderImage,
		UsingDefaultImage,
		UsingTargetImage,
	};

	/** Which file are we using? */
	EFileLocation TypeOfImage;

	/** The dimensions the image display should be constrained to */
	FVector2D MaxDisplayedImageDimensions;

	/** The dimensions the image ought to be (if bRequiresSpecificSize is set) */
	FIntPoint RequiredImageDimensions;

	/** Does the image need to be a specific size? */
	bool bRequiresSpecificSize;

	/** The error hint widget used to display bad sizes */
	TSharedPtr<class SErrorText> ErrorHintWidget;
};
