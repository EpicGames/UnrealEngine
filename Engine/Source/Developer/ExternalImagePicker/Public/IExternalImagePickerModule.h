// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "Framework/SlateDelegates.h"

/**
 * Delegate fired when picking a new image.
 * @param	InChosenImage	The path to the image that has been picked by the user.
 * @param	InTargetImage	The path to the target image that this picker represents.
 * @return true if the image should be refreshed
 */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnExternalImagePicked, const FString& /** InChosenImage */, const FString& /** InTargetImage */);

/**
 * Delegate fired to get the path to start picking from.
 * @return the path the picker will use to start from
 */
DECLARE_DELEGATE_RetVal(FString, FOnGetPickerPath);

/**
 * Used for configuring the external image picker
 */
struct FExternalImagePickerConfiguration
{
	FExternalImagePickerConfiguration()
		: MaxDisplayedImageDimensions(400.0f, 400.0f)
		, bRequiresSpecificSize(false)
	{
	}

	/** The image on disk that the external image is stored as. */
	FString TargetImagePath;

	/** The image on disk that we will use if the target does not exist. */
	FString DefaultImagePath;

	/** The path the picker will use to start from */
	FOnGetPickerPath OnGetPickerPath;

	/** Delegate fired when picking a new image. */
	FOnExternalImagePicked OnExternalImagePicked;

	/** The dimensions the image display should be constrained to. Aspect ratio is maintained. */
	FVector2D MaxDisplayedImageDimensions;

	/** The size the actual image needs to be (ignored unless bRequiresSpecificSize is set) */
	FIntPoint RequiredImageDimensions;

	/** Does the image need to be a specific size? */
	bool bRequiresSpecificSize;

	/** The image on disk that we will use if the target does not exist. */
	TArray<FString> FileExtensions;
    
	/** Whether the button to generate a new image should be shown. */
	TAttribute<EVisibility> GenerateImageVisibility;

	/** Tooltip for Generate Image button */
	FText GenerateImageToolTipText;

	/** Delegate fired when an Generate Image button is pressed */
	FOnClicked OnGenerateImageClicked;

	/** A property handle to use if required */
	TSharedPtr<class IPropertyHandle> PropertyHandle;
};

/**
 * Public interface for splash screen editor module
 */
class IExternalImagePickerModule : public IModuleInterface
{
public:
	/**
	 * Make a widget used for displaying & editing splash screens
	 *
	 * @param	
	 * @return the image editing widget
	 */
	virtual TSharedRef<class SWidget> MakeEditorWidget(const FExternalImagePickerConfiguration& InConfiguration) = 0;

	/**
	 * Gets a reference to the ExternalImagePicker module instance.
	 *
	 * @return A reference to the ExternalImagePicker module.
	 */
	static IExternalImagePickerModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IExternalImagePickerModule>("ExternalImagePicker");
	}

};
